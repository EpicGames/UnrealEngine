{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Editor Telemetry",
	"Description": "Plugin that emits common telemetry events from the Editor",
	"Category": "Telemetry",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "RuntimeTelemetry",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "PostConfigInit"
		}
	]
}