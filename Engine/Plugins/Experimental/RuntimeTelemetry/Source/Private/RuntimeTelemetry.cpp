// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeTelemetry.h"
#include "StudioTelemetry.h"
#include "Misc/CoreDelegates.h"
#include "Containers/Map.h"
#include "HAL/LowLevelMemTracker.h"
#include "IO/IoStoreOnDemand.h"

DEFINE_LOG_CATEGORY(LogRuntimeTelemetry);

FRuntimeTelemetry& FRuntimeTelemetry::Get()
{
	static FRuntimeTelemetry RuntimeTelemetryInstance;
	return RuntimeTelemetryInstance;
}

void FRuntimeTelemetry::RecordEvent_IoStoreOnDemand(const FString& Context, TArray<FAnalyticsEventAttribute> Attributes)
{
	if (FStudioTelemetry::Get().IsSessionRunning())
	{
		if (UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore())
		{
			// Only record event data if IoStoreOnDemand is enabled
			if (IoStore->IsOnDemandStreamingEnabled())
			{
				const int SchemaVersion = 1;

				TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

				IoStore->ReportAnalytics(EventAttributes);

				EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
				EventAttributes.Emplace(TEXT("Context"), Context);

				FStudioTelemetry::Get().RecordEvent("Core.IoStoreOnDemand", EventAttributes);
				FStudioTelemetry::Get().FlushEvents();
			}
		}
	}
}


void FRuntimeTelemetry::RecordEvent_MemoryLLM(const FString& Context, TArray<FAnalyticsEventAttribute> Attributes)
{
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (FStudioTelemetry::Get().IsSessionRunning())
	{
		auto RecordLLMMemoryEvent = [&Attributes](const FString& Context, const FString& TagSet, TMap<FName, uint64>& LLMTrackedMemoryMap)
			{
				for (TMap<FName, uint64>::TConstIterator It(LLMTrackedMemoryMap); It; ++It)
				{
					const int SchemaVersion = 2;

					TArray<FAnalyticsEventAttribute> EventAttributes = Attributes;

					EventAttributes.Emplace(TEXT("SchemaVersion"), SchemaVersion);
					EventAttributes.Emplace(TEXT("Context"), Context);
					EventAttributes.Emplace(TEXT("TagSet"), TagSet);
					EventAttributes.Emplace(TEXT("Name"), It->Key);
					EventAttributes.Emplace(TEXT("Size"), It->Value);

					FStudioTelemetry::Get().RecordEvent("Core.Memory.LLM", EventAttributes);
				}
			};

		// None TagSet
		TMap<FName, uint64> LLMTrackedNoneMemory;
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmount(LLMTrackedNoneMemory, ELLMTracker::Default, ELLMTagSet::None);
		RecordLLMMemoryEvent(Context, TEXT("None"), LLMTrackedNoneMemory);

		// AssetClasses TagSet
		TMap<FName, uint64> LLMTrackedAssetClassesMemory;
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmount(LLMTrackedAssetClassesMemory, ELLMTracker::Default, ELLMTagSet::AssetClasses);
		RecordLLMMemoryEvent(Context, TEXT("AssetClasses"), LLMTrackedAssetClassesMemory);

		// Asset TagSet	
		TMap<FName, uint64> LLMTrackedAssetMemory;
		FLowLevelMemTracker::Get().GetTrackedTagsNamesWithAmount(LLMTrackedAssetMemory, ELLMTracker::Default, ELLMTagSet::Assets);
		RecordLLMMemoryEvent(Context, TEXT("Assets"), LLMTrackedAssetMemory);
	}
#endif
}
void FRuntimeTelemetry::StartSession()
{
	// Register a call back to intercept the point where the game is shutting down
	FCoreDelegates::OnEnginePreExit.AddLambda([this]()
		{
			UE_LOGF(LogRuntimeTelemetry, Log, "Recording EnginePreExit events");

			RecordEvent_IoStoreOnDemand(TEXT("EnginePreExit"));
			RecordEvent_MemoryLLM(TEXT("EnginePreExit"));
		}
	);
}

void FRuntimeTelemetry::EndSession()
{
}
