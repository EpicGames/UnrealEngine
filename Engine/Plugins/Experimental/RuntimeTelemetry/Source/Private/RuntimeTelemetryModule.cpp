// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeTelemetryModule.h"
#include "RuntimeTelemetry.h"
#include "StudioTelemetry.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FRuntimeTelemetryModule, RuntimeTelemetry);

void FRuntimeTelemetryModule::StartupModule()
{
#if !UE_BUILD_SHIPPING
	FStudioTelemetry::Get().StartSession();
	FRuntimeTelemetry::Get().StartSession();	
#endif
}

void FRuntimeTelemetryModule::ShutdownModule()
{
#if !UE_BUILD_SHIPPING
	FRuntimeTelemetry::Get().EndSession();
	FStudioTelemetry::Get().EndSession();
#endif
}
