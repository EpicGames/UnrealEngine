// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAnalyticsProvider.h"
#include "Interfaces/IAnalyticsTracer.h"
#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"

#define UE_API RUNTIMETELEMETRY_API

DECLARE_LOG_CATEGORY_EXTERN(LogRuntimeTelemetry, Log, All);

/**
 * 
 */
class FRuntimeTelemetry
{
public:
	FRuntimeTelemetry() = default;
	~FRuntimeTelemetry() = default;

	static UE_API FRuntimeTelemetry& Get();

	UE_API void StartSession();
	UE_API void EndSession();

	/** Useful event recording functions */
	UE_API void RecordEvent_IoStoreOnDemand(const FString& Context, TArray<FAnalyticsEventAttribute> Attributes = {});
	UE_API void RecordEvent_MemoryLLM(const FString& Context, TArray<FAnalyticsEventAttribute> Attributes = {});
	
private:	

};



#undef UE_API
